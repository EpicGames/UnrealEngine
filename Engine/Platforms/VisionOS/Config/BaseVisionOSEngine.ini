[/Script/MacTargetPlatform.XcodeProjectSettings]
bUseSwiftUIMain=true
