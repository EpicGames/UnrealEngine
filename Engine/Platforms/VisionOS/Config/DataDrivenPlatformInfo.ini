[DataDrivenPlatformInfo]
bIsConfidential=false
IniParent=IOS
TargetSettingsIniSectionName=/Script/IOSRuntimeSettings.IOSRuntimeSettings
GlobalIdentifier=9981B55362F64CF9957058C72123A102

NormalIconPath=/Platforms/VisionOS/Content/Editor/Slate/Launcher/Platform_VisionOS_24x
LargeIconPath=/Platforms/VisionOS/Content/Editor/Slate/Launcher/Platform_VisionOS_128x
XLargeIconPath=
AutoSDKPath=
TutorialPath=SharingAndReleasing/Mobile/iOS
Windows:bIsEnabled=false
Mac:bIsEnabled=true
Linux:bIsEnabled=false
bUsesHostCompiler=false
bUATClosesAfterLaunch=true
PlatformGroupName=Mobile
ICUDataVersion=icudt64l

