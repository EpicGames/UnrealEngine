; NOTE: most values should be added to BaseVisionOSEngine.ini unless there is a very specific reason for it
; items added here will take priority over the project's DefaultEngine.ini. Use the project's VisionOSEngine.ini to override values here.

; These settings should be considered 'recommended platform defaults'

[/Script/Engine.RendererSettings]
; These settings are always required for VisionOS, so we override any other project settings.
vr.InstancedStereo=False
vr.MobileMultiView=False
xr.OpenXRAcquireMode=1

// These are required for .mixed mode, aka AR
;r.Mobile.PropagateAlpha=1
;r.PostProcessing.PropagateAlpha=1
;r.AlphaInvertPass=1

