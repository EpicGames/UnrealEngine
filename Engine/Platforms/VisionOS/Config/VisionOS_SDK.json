{
	// inherit the IOS MinSoftwareVersion
	"ParentSDKFile": "../../../Config/IOS/IOS_SDK.json",

	// xcode version
	"MinVersion": "15.3.0",
	
	// runtime OS version
	"MinSoftwareVersion":  "1.0"
}
