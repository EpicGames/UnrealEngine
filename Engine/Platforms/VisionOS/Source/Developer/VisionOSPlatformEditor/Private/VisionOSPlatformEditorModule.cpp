// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISettingsModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "VisionOSRuntimeSettings.h"

#define LOCTEXT_NAMESPACE "FVisionOSPlatformEditorModule"


/**
 * Module for VisionOS as a target platform
 */
class FVisionOSPlatformEditorModule
	: public IModuleInterface
{
	// IModuleInterface interface

	virtual void StartupModule() override
	{
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "VisionOS",
				LOCTEXT("VisionOSSettingsName", "visionOS"),
				LOCTEXT("VisionOSSettingsDescription", "Settings for visionOS projects"),
				GetMutableDefault<UVisionOSRuntimeSettings>());

		}
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "VisionOS");
		}
	}
};


IMPLEMENT_MODULE(FVisionOSPlatformEditorModule, VisionOSPlatformEditor);

#undef LOCTEXT_NAMESPACE
