// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VisionOSPlatformEditor : ModuleRules
{
	public VisionOSPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "VisionOS";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"PropertyEditor",
				"SlateCore",
				"VisionOSRuntimeSettings",
            }
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
				}
		);
	}
}
