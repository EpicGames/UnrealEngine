// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VisionOSTargetPlatform : ModuleRules
{
	public VisionOSTargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "IOS";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.VisionOS);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TargetPlatform",
				"DesktopPlatform",
			}
		);
	}
}
