// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetPlatformControls.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"

/**
 * Module for TVOS as a target platform controls
 */
class FVisionOSTargetPlatformControlsModule	: public ITargetPlatformControlsModule
{
public:
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) override
	{
		if (FIOSTargetPlatformControls::IsUsable())
		{
			ITargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<ITargetPlatformSettingsModule>(PlatformSettingsModuleName);
			if (ModuleSettings != nullptr)
			{
				TArray<ITargetPlatformSettings*> TargetPlatformSettings = ModuleSettings->GetTargetPlatformSettings();
				check(TargetPlatformSettings.Num() == 1);

				TargetPlatforms.Add(new FIOSTargetPlatformControls(false, true, false, TargetPlatformSettings[0]));
				TargetPlatforms.Add(new FIOSTargetPlatformControls(false, true, true, TargetPlatformSettings[0]));
			}
		}
	}
};


IMPLEMENT_MODULE(FVisionOSTargetPlatformControlsModule, VisionOSTargetPlatformControls);
