// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSTargetPlatformSettings.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"

/**
 * Module for TVOS as a target platform settings
 */
class FVisionOSTargetPlatformSettingsModule	: public ITargetPlatformSettingsModule
{
public:
	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms) override
	{
		TargetPlatforms.Add(new FIOSTargetPlatformSettings(false, true));
	}
};


IMPLEMENT_MODULE(FVisionOSTargetPlatformSettingsModule, VisionOSTargetPlatformSettings);
