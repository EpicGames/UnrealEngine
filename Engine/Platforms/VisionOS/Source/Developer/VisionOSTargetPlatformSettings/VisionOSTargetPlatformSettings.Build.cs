// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VisionOSTargetPlatformSettings : ModuleRules
{
	public VisionOSTargetPlatformSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "IOS";
		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "VisionOSTPSet";

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
				"IOSTargetPlatformSettings",
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Engine");
		}

		PrivateIncludePaths.AddRange(
			new string[] {
			"Developer/IOS/IOSTargetPlatformSettings/Private"
			}
		);
	}
}
