// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

class XcodeTargetPlatform_VisionOS : XcodeTargetPlatform_IOS
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.VisionOS);
	public override bool IsPlatformExtension => true;
	protected override string CMakeSystemName => "VisionOS";
}

class MakefileTargetPlatform_VisionOS : MakefileTargetPlatform_IOS
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.VisionOS);
	public override bool IsPlatformExtension => true;
	protected override string CMakeSystemName => "VisionOS";
	protected override string SdkName => Architecture.Contains("simulator") || Architecture == "x86_64" ? "xrsimulator" : "xros";

	public MakefileTargetPlatform_VisionOS(string Architecture) : base(Architecture)
	{
	}

}

class NinjaBuildTargetPlatform_VisionOS : NinjaBuildTargetPlatform_IOS
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.VisionOS);
	public override bool IsPlatformExtension => true;
	protected override string CMakeSystemName => "VisionOS";
	protected override string SdkName => Architecture.Contains("simulator") || Architecture == "x86_64" ? "xrsimulator" : "xros";

	public NinjaBuildTargetPlatform_VisionOS(string Architecture) : base(Architecture)
	{
	}

}
