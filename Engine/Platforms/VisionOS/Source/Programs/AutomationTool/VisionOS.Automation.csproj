<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\..\..\..\..\Engine\Source\Programs\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>VisionOS.Automation</RootNamespace>
    <AssemblyName>VisionOS.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="..\..\..\..\..\Source\Programs\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\..\..\..\Source\Programs\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\..\..\..\Source\Programs\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\..\..\..\Source\Programs\AutomationTool\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\..\..\..\Source\Programs\AutomationTool\Scripts\AutomationScripts.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\..\..\..\Source\Programs\AutomationTool\IOS\IOS.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
</Project>