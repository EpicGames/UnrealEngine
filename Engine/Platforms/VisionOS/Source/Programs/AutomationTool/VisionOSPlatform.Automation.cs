// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using EpicGames.Core;

public class VisionOSPlatform : IOSPlatform
{
	public VisionOSPlatform()
		:base(UnrealTargetPlatform.VisionOS)
	{
		TargetIniPlatformType = UnrealTargetPlatform.IOS;
	}

	public override bool PrepForUATPackageOrDeploy(UnrealTargetConfiguration Config, FileReference ProjectFile, string InProjectName, DirectoryReference InProjectDirectory, FileReference Executable, DirectoryReference InEngineDir, bool bForDistribution, string CookFlavor, bool bIsDataDeploy, bool bCreateStubIPA, bool bIsUEGame)
	{
		string TargetName = Path.GetFileNameWithoutExtension(Executable.FullName).Split("-".ToCharArray())[0];
		FileReference TargetReceiptFileName;
		if (bIsUEGame)
		{
			TargetReceiptFileName = TargetReceipt.GetDefaultPath(InEngineDir, "UnrealGame", UnrealTargetPlatform.VisionOS, Config, null);
		}
		else
		{
			TargetReceiptFileName = TargetReceipt.GetDefaultPath(InProjectDirectory, TargetName, UnrealTargetPlatform.VisionOS, Config, null);
		}
		return VisionOSExports.PrepForUATPackageOrDeploy(Config, ProjectFile, InProjectName, InProjectDirectory, Executable, InEngineDir, bForDistribution, CookFlavor, bIsDataDeploy, bCreateStubIPA, TargetReceiptFileName, Log.Logger);
	}

	public override bool DeployGeneratePList(FileReference ProjectFile, UnrealTargetConfiguration Config, DirectoryReference ProjectDirectory, bool bIsUEGame, string GameName, bool bIsClient, string ProjectName, DirectoryReference InEngineDir, DirectoryReference AppDirectory, TargetReceipt Receipt)
	{
		return VisionOSExports.GeneratePList(ProjectFile, Config, ProjectDirectory, bIsUEGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, Receipt, Log.Logger);
	}

    public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return bIsClientOnly ? "VisionOSClient" : "VisionOS";
	}

    public override void GetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
    {
        //		if (UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
        {
            // copy the icons/launch screens from the engine
            {
				FileReference SourcePath = FileReference.Combine(SC.LocalRoot, "Engine", "Binaries", "VisionOS", "AssetCatalog", "Assets.car");
				if(FileReference.Exists(SourcePath))
				{
					SC.StageFile(StagedFileType.SystemNonUFS, SourcePath, new StagedFileReference("Assets.car"));
				}
            }

            // copy any additional framework assets that will be needed at runtime
            {
                DirectoryReference SourcePath = DirectoryReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : SC.EngineRoot), "Intermediate", "VisionOS", "FrameworkAssets");
                if (DirectoryReference.Exists(SourcePath))
                {
					SC.StageFiles(StagedFileType.SystemNonUFS, SourcePath, StageFilesSearch.AllDirectories, StagedDirectoryReference.Root);
                }
            }

            // copy the icons/launch screens from the game (may stomp the engine copies)
            {
                FileReference SourcePath = FileReference.Combine(SC.ProjectRoot, "Binaries", "VisionOS", "AssetCatalog", "Assets.car");
				if(FileReference.Exists(SourcePath))
				{
	                SC.StageFile(StagedFileType.SystemNonUFS, SourcePath, new StagedFileReference("Assets.car"));
				}
            }

            // copy the plist (only if code signing, as it's protected by the code sign blob in the executable and can't be modified independently)
            if (GetCodeSignDesirability(Params))
            {
				// this would be FooClient when making a client-only build
				string TargetName = SC.StageExecutables[0].Split("-".ToCharArray())[0];
				DirectoryReference SourcePath = DirectoryReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : SC.EngineRoot), "Intermediate", "VisionOS");
                FileReference TargetPListFile = FileReference.Combine(SourcePath, (SC.IsCodeBasedProject ? TargetName : "UnrealGame") + "-Info.plist");
                //				if (!File.Exists(TargetPListFile))
                {
                    // ensure the plist, entitlements, and provision files are properly copied
                    Console.WriteLine("CookPlat {0}, this {1}", GetCookPlatform(false, false), ToString());
                    if (!SC.IsCodeBasedProject)
                    {
                        UnrealBuildTool.PlatformExports.SetRemoteIniPath(SC.ProjectRoot.FullName);
                    }

                    if (SC.StageTargetConfigurations.Count != 1)
                    {
                        throw new AutomationException("iOS is currently only able to package one target configuration at a time, but StageTargetConfigurations contained {0} configurations", SC.StageTargetConfigurations.Count);
                    }

                    var TargetConfiguration = SC.StageTargetConfigurations[0];

                    DeployGeneratePList(
						SC.RawProjectPath,
						TargetConfiguration,
						(SC.IsCodeBasedProject ? SC.ProjectRoot : SC.EngineRoot),
						!SC.IsCodeBasedProject,
						(SC.IsCodeBasedProject ? TargetName : "UnrealGame"),
						Params.Client,
						SC.ShortProjectName,
						SC.EngineRoot,
						DirectoryReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : SC.EngineRoot),
						"Binaries",
						"VisionOS",
						"Payload",
						(SC.IsCodeBasedProject ? SC.ShortProjectName : "UnrealGame") + ".app"),
						SC.StageTargets[0].Receipt);
                }


            // copy the udebugsymbols if they exist
            {
                ConfigHierarchy PlatformGameConfig;
                bool bIncludeSymbols = false;
                if (Params.EngineConfigs.TryGetValue(SC.StageTargetPlatform.PlatformType, out PlatformGameConfig))
                {
                    PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGenerateCrashReportSymbols", out bIncludeSymbols);
                }
                if (bIncludeSymbols)
                {
                    FileReference SymbolFileName = FileReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : SC.EngineRoot), "Binaries", "VisionOS", SC.StageExecutables[0] + ".udebugsymbols");
                    if (FileReference.Exists(SymbolFileName))
                    {
                        SC.StageFile(StagedFileType.NonUFS, SymbolFileName, new StagedFileReference((Params.ShortProjectName + ".udebugsymbols").ToLowerInvariant()));
                    }
                }
            }
                SC.StageFile(StagedFileType.SystemNonUFS, TargetPListFile, new StagedFileReference("Info.plist"));
            }
        }

        // copy the movies from the project
        {
            StageMovieFiles(DirectoryReference.Combine(SC.EngineRoot, "Content", "Movies"), SC);
            StageMovieFiles(DirectoryReference.Combine(SC.ProjectRoot, "Content", "Movies"), SC);
        }

        {
            // Get the final output directory for cooked data
            DirectoryReference CookOutputDir;
            if (!String.IsNullOrEmpty(Params.CookOutputDir))
            {
                CookOutputDir = DirectoryReference.Combine(new DirectoryReference(Params.CookOutputDir), SC.CookPlatform);
            }
            else if (Params.CookInEditor)
            {
                CookOutputDir = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "EditorCooked", SC.CookPlatform);
            }
            else
            {
                CookOutputDir = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "Cooked", SC.CookPlatform);
            }
        }
    }
}
