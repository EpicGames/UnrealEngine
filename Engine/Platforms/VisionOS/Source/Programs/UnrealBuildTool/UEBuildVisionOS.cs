// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class VisionOSProjectSettings : IOSProjectSettings
	{
		// @todo get these from ini or whatever
		public override string RuntimeDevices => "7";
		public override string RuntimeVersion => UEBuildPlatformSDK.GetSDKForPlatform("VisionOS")!.GetSoftwareInfo()!.Min!;

		public VisionOSProjectSettings(FileReference? ProjectFile, string? Bundle)
			: base(ProjectFile, UnrealTargetPlatform.VisionOS, Bundle)
		{
		}
	}

	class VisionOSPlatform : IOSPlatform
	{
		// Cached VisionOS sdk version from the toolchain, which detects it.
		private float SDKVersionFloat = 0.0f;

		public VisionOSPlatform(UEBuildPlatformSDK InSDK, ILogger Logger)
			: base(InSDK, UnrealTargetPlatform.VisionOS, Logger)
		{
		}

		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);

			// make sure we add Metal, in case base class got it wrong
			if (Target.GlobalDefinitions.Contains("HAS_METAL=0"))
			{
				Target.GlobalDefinitions.Remove("HAS_METAL=0");
				Target.GlobalDefinitions.Add("HAS_METAL=1");
				Target.ExtraModuleNames.Add("MetalRHI");
			}
		}

		public new VisionOSProjectSettings ReadProjectSettings(FileReference? ProjectFile, string Bundle = "")
		{
			return (VisionOSProjectSettings)base.ReadProjectSettings(ProjectFile, Bundle);
		}

		protected override IOSProjectSettings CreateProjectSettings(FileReference? ProjectFile, string? Bundle)
		{
			return new VisionOSProjectSettings(ProjectFile, Bundle);
		}

		public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			base.ModifyModuleRulesForOtherPlatform(ModuleName, Rules, Target);

			bool bIsPlatformAvailableForTarget = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target, bIgnoreSDKCheck: true);
			bool bIsPlatformAvailableForTargetWithSDK = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target);
			// don't do any target platform stuff if SDK is not available
			if (!bIsPlatformAvailableForTarget)
			{
				return;
			}

			if (!Target.bBuildRequiresCookedData)
			{
				if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Win64)
				{
					if ((ModuleName == "Engine" && Target.bBuildDeveloperTools) ||
						(ModuleName == "TargetPlatform" && Target.bForceBuildTargetPlatforms))
					{
						Rules.DynamicallyLoadedModuleNames.Add("VisionOSTargetPlatformSettings");
						if (bIsPlatformAvailableForTargetWithSDK)
						{
							Rules.DynamicallyLoadedModuleNames.Add("VisionOSTargetPlatform");
							Rules.DynamicallyLoadedModuleNames.Add("VisionOSTargetPlatformControls");
						}
					}
				}
			}

			if (ModuleName == "UnrealEd" && bIsPlatformAvailableForTargetWithSDK)
			{
				Rules.DynamicallyLoadedModuleNames.Add("VisionOSPlatformEditor");
			}
		}

		/// <summary>
		/// Setup the target environment for building
		/// </summary>
		/// <param name="Target">Settings for the target being compiled</param>
		/// <param name="CompileEnvironment">The compile environment for this target</param>
		/// <param name="LinkEnvironment">The link environment for this target</param>
		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			base.SetUpEnvironment(Target, CompileEnvironment, LinkEnvironment);
			CompileEnvironment.Definitions.Add("PLATFORM_VISIONOS=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_VISIONOS=1");

			if (SDKVersionFloat < 2.0)
			{
				CompileEnvironment.Definitions.Add("VISIONOS_MAJOR_VERSION=1");
			}
			else
			{
				CompileEnvironment.Definitions.Add("VISIONOS_MAJOR_VERSION=2");
			}

			// VisionOS uses only IOS header files, so use it's platform headers
			CompileEnvironment.Definitions.Add("OVERRIDE_PLATFORM_HEADER_NAME=IOS");
		}

		/// <summary>
		/// Creates a toolchain instance for the given platform.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>New toolchain instance.</returns>
		public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
		{
			VisionOSProjectSettings ProjectSettings = ((VisionOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.VisionOS)).ReadProjectSettings(Target.ProjectFile);
			VisionOSToolChain NewToolChain = new VisionOSToolChain(Target, ProjectSettings, Logger);
			SDKVersionFloat = NewToolChain.SDKVersionFloat;
			return NewToolChain;
		}

		public override void Deploy(TargetReceipt Receipt)
		{
			new UEDeployVisionOS(Logger).PrepTargetForDeployment(Receipt);
		}
	}

	class VisionOSPlatformFactory : UEBuildPlatformFactory
	{
		public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.VisionOS;

		/// <summary>
		/// Register the platform with the UEBuildPlatform class
		/// </summary>
		public override void RegisterBuildPlatforms(ILogger Logger)
		{
			ApplePlatformSDK SDK = new(Logger);

			// Register this build platform for IOS
			UEBuildPlatform.RegisterBuildPlatform(new VisionOSPlatform(SDK, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.VisionOS, UnrealPlatformGroup.Apple);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.VisionOS, UnrealPlatformGroup.IOS);
		}
	}
}

