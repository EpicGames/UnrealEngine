// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Xml.Linq;

namespace UnrealBuildTool
{
	class UEDeployVisionOS : UEDeployIOS
	{

		public UEDeployVisionOS(ILogger InLogger)
			: base(InLogger)
		{
		}

		protected override string GetTargetPlatformName()
		{
			return "VisionOS";
		}

		public static bool GenerateVisionOSPList(string ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, string InEngineDir, string AppDirectory, UnrealPluginLanguage? UPL, string? BundleID)
		{
			// @todo visionos: THIS!

			// generate the Info.plist for future use
			string BuildDirectory = ProjectDirectory + "/Build/VisionOS";
			bool bSkipDefaultPNGs = false;
			string IntermediateDirectory = (bIsUnrealGame ? InEngineDir : ProjectDirectory) + "/Intermediate/VisionOS";
			string PListFile = IntermediateDirectory + "/" + GameName + "-Info.plist";
			// @todo tvos: This is really nasty - both IOS and TVOS are setting static vars
			VersionUtilities.BuildDirectory = BuildDirectory;
			VersionUtilities.GameName = GameName;

			// read the old file
			string OldPListData = File.Exists(PListFile) ? File.ReadAllText(PListFile) : "";

			// get the settings from the ini file
			// plist replacements
			// @todo tvos: Are we going to make TVOS specific .ini files?
			DirectoryReference? DirRef = bIsUnrealGame ? (!String.IsNullOrEmpty(UnrealBuildTool.GetRemoteIniPath()) ? new DirectoryReference(UnrealBuildTool.GetRemoteIniPath()!) : null) : new DirectoryReference(ProjectDirectory);
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirRef, UnrealTargetPlatform.IOS);

			// bundle display name
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleDisplayName", out string BundleDisplayName);

			// bundle identifier
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out string BundleIdentifier);
			if (!String.IsNullOrEmpty(BundleID))
			{
				BundleIdentifier = BundleID;
			}

			// bundle name
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleName", out string BundleName);

			// short version string
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "VersionInfo", out string BundleShortVersion);

			// required capabilities
			string RequiredCaps = "\t\t<string>arm64</string>\n";

			// @todo Handle visionOS minimum supported version properly.
			//Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "MinimumiOSVersion", out string MinVersionSetting);
			//string MinVersion = GetMinimumOSVersion(MinVersionSetting, Logger);

			// @todo Handle visionOS additional plist data properly.
			//Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "AdditionalPlistData", out string ExtraData);

			// generate the plist file
			StringBuilder Text = new();
			Text.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
			Text.AppendLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
			Text.AppendLine("<plist version=\"1.0\">");
			Text.AppendLine("<dict>");
			Text.AppendLine("\t<key>CFBundleDevelopmentRegion</key>");
			Text.AppendLine("\t<string>en</string>");
			Text.AppendLine("\t<key>CFBundleDisplayName</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", EncodeBundleName(BundleDisplayName, ProjectName)));
			Text.AppendLine("\t<key>CFBundleExecutable</key>");
			string BundleExecutable = bIsUnrealGame ?
				(bIsClient ? "UnrealClient" : "UnrealGame") :
				(bIsClient ? GameName + "Client" : GameName);
			Text.AppendLine(String.Format("\t<string>{0}</string>", BundleExecutable));
			Text.AppendLine("\t<key>CFBundleIdentifier</key>");
			string SanitizedBundleIdentifier = BundleIdentifier
				.Replace("[PROJECT_NAME]", ProjectName, StringComparison.Ordinal)
				.Replace("_", "", StringComparison.Ordinal);
			Text.AppendLine($"\t<string>{SanitizedBundleIdentifier}</string>");
			Text.AppendLine("\t<key>CFBundleInfoDictionaryVersion</key>");
			Text.AppendLine("\t<string>6.0</string>");
			Text.AppendLine("\t<key>CFBundleName</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", EncodeBundleName(BundleName, ProjectName)));
			Text.AppendLine("\t<key>CFBundlePackageType</key>");
			Text.AppendLine("\t<string>APPL</string>");
			Text.AppendLine("\t<key>CFBundleSignature</key>");
			Text.AppendLine("\t<string>????</string>");
			Text.AppendLine("\t<key>CFBundleVersion</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", VersionUtilities.UpdateBundleVersion(OldPListData)));
			Text.AppendLine("\t<key>CFBundleShortVersionString</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", BundleShortVersion));
			Text.AppendLine("\t<key>LSRequiresIPhoneOS</key>");
			Text.AppendLine("\t<true/>");
			Text.AppendLine("\t<key>UIRequiredDeviceCapabilities</key>");
			Text.AppendLine("\t<array>");
			foreach (string Line in RequiredCaps.Split("\r\n".ToCharArray()))
			{
				if (!String.IsNullOrWhiteSpace(Line))
				{
					Text.AppendLine(Line);
				}
			}
			Text.AppendLine("\t</array>");

			Text.AppendLine("\t<key>TVTopShelfImage</key>");
			Text.AppendLine("\t<dict>");
			Text.AppendLine("\t\t<key>TVTopShelfPrimaryImageWide</key>");
			Text.AppendLine("\t\t<string>Top Shelf Image Wide</string>");
			Text.AppendLine("\t</dict>");
			Text.AppendLine("\t<key>CFBundleIcons</key>");
			Text.AppendLine("\t<dict>");
			Text.AppendLine("\t\t<key>CFBundlePrimaryIcon</key>");
			Text.AppendLine("\t\t<string>App Icon</string>");
			Text.AppendLine("\t</dict>");
			Text.AppendLine("\t<key>UILaunchStoryboardName</key>");
			Text.AppendLine("\t<string>LaunchScreen</string>");

			// write the iCloud container identifier, if present in the old file
			if (!String.IsNullOrEmpty(OldPListData))
			{
				int index = OldPListData.IndexOf("ICloudContainerIdentifier", StringComparison.Ordinal);
				if (index > 0)
				{
					index = OldPListData.IndexOf("<string>", index, StringComparison.Ordinal) + 8;
					int length = OldPListData.IndexOf("</string>", index, StringComparison.Ordinal) - index;
					string ICloudContainerIdentifier = OldPListData.Substring(index, length);
					Text.AppendLine("\t<key>ICloudContainerIdentifier</key>");
					Text.AppendLine(String.Format("\t<string>{0}</string>", ICloudContainerIdentifier));
				}
			}

			Text.AppendLine("</dict>");
			Text.AppendLine("</plist>");

			// Create the intermediate directory if needed
			if (!Directory.Exists(IntermediateDirectory))
			{
				Directory.CreateDirectory(IntermediateDirectory);
			}

			if (UPL != null)
			{
				// Allow UPL to modify the plist here
				XDocument XDoc;
				try
				{
					XDoc = XDocument.Parse(Text.ToString());
				}
				catch (Exception e)
				{
					throw new BuildException("plist is invalid {0}\n{1}", e, Text.ToString());
				}

				XDoc.DocumentType!.InternalSubset = "";
				UPL.ProcessPluginNode("None", "iosPListUpdates", "", ref XDoc);
				string result = XDoc.Declaration?.ToString() + "\n" + XDoc.ToString().Replace(
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\"[]>",
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">",
					StringComparison.Ordinal);
				File.WriteAllText(PListFile, result);
			}
			else
			{
				File.WriteAllText(PListFile, Text.ToString());
			}

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				if (!Directory.Exists(AppDirectory))
				{
					Directory.CreateDirectory(AppDirectory);
				}
				File.WriteAllText(AppDirectory + "/Info.plist", Text.ToString());
			}

			return bSkipDefaultPNGs;
		}

		public override bool GeneratePList(FileReference? ProjectFile, UnrealTargetConfiguration Config, string ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, string InEngineDir, string AppDirectory, List<string> UPLScripts, string? BundleID, bool bBuildAsFramework)
		{
			return GenerateVisionOSPList(ProjectDirectory, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, null, BundleID);
		}
	}
}
