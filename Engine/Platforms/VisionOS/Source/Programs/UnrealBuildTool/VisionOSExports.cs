// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public VisionOS functions exposed to UAT
	/// </summary>
	public static class VisionOSExports
	{
		/// <summary>
		/// 
		/// </summary>
		/// <param name="Config"></param>
		/// <param name="ProjectFile"></param>
		/// <param name="InProjectName"></param>
		/// <param name="InProjectDirectory"></param>
		/// <param name="Executable"></param>
		/// <param name="InEngineDir"></param>
		/// <param name="bForDistribution"></param>
		/// <param name="CookFlavor"></param>
		/// <param name="bIsDataDeploy"></param>
		/// <param name="bCreateStubIPA"></param>
		/// <param name="BuildReceiptFileName"></param>
		/// <param name="Logger"></param>
		/// <returns></returns>
		public static bool PrepForUATPackageOrDeploy(UnrealTargetConfiguration Config, FileReference ProjectFile, string InProjectName, DirectoryReference InProjectDirectory, FileReference Executable, DirectoryReference InEngineDir, bool bForDistribution, string CookFlavor, bool bIsDataDeploy, bool bCreateStubIPA, FileReference BuildReceiptFileName, ILogger Logger)
		{
			TargetReceipt Receipt = TargetReceipt.Read(BuildReceiptFileName);
			return new UEDeployVisionOS(Logger).PrepForUATPackageOrDeploy(Config, ProjectFile, InProjectName, InProjectDirectory.FullName, Executable, InEngineDir.FullName, bCreateStubIPA, Receipt);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <param name="Config"></param>
		/// <param name="ProjectDirectory"></param>
		/// <param name="bIsUnrealGame"></param>
		/// <param name="GameName"></param>
		/// <param name="bIsClient"></param>
		/// <param name="ProjectName"></param>
		/// <param name="InEngineDir"></param>
		/// <param name="AppDirectory"></param>
		/// <param name="Receipt"></param>
		/// <param name="Logger"></param>
		/// <returns></returns>
		public static bool GeneratePList(FileReference ProjectFile, UnrealTargetConfiguration Config, DirectoryReference ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, DirectoryReference InEngineDir, DirectoryReference AppDirectory, TargetReceipt Receipt, ILogger Logger)
		{
			return new UEDeployVisionOS(Logger).GeneratePList(ProjectFile, Config, ProjectDirectory.FullName, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir.FullName, AppDirectory.FullName, Receipt);
		}
	}
}
