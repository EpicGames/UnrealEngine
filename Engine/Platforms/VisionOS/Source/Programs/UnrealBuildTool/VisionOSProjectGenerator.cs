// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;

namespace UnrealBuildTool
{
	class VisionOSProjectGenerator : IOSProjectGenerator
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Arguments">Command line arguments passed to the project generator</param>
		/// <param name="Logger">Logger for output</param>
		public VisionOSProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Arguments, Logger)
		{
		}

		/// <summary>
		/// Enumerate all the platforms that this generator supports
		/// </summary>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.VisionOS;
		}
	}
}
