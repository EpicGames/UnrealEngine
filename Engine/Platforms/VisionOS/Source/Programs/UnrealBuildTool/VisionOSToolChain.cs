// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class VisionOSToolChain : IOSToolChain
	{
		protected override string XnuPlatformDefine => "XNU_PLATFORM_XROS";

		public VisionOSToolChain(ReadOnlyTargetRules InTarget, VisionOSProjectSettings InProjectSettings, ILogger InLogger)
			: base(InTarget, InProjectSettings, () => new VisionOSToolChainSettings(InProjectSettings, InLogger), ClangToolChainOptions.None, InLogger)
		{
		}
	}
}
