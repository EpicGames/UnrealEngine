// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class VisionOSToolChainSettings : IOSToolChainSettings
	{
		public VisionOSToolChainSettings(IOSProjectSettings ProjectSettings, ILogger Logger)
			: base("XROS", "XRSimulator", "xros", ProjectSettings, Logger)
		{
		}
	}
}
