// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

public class Launch_VisionOS : Launch
{
	protected override bool bAllowSwiftMain => true;

	public Launch_VisionOS(ReadOnlyTargetRules Target) : base(Target)
	{
	}
}
