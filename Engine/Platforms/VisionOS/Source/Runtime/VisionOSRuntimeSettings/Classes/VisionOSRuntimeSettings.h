// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Logging/LogMacros.h"

#include "VisionOSRuntimeSettings.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogVisionOSRuntimeSettings, Log, All);

/**
 * Implements the settings for the VisionOS target platform.
 */
UCLASS(config=Engine, defaultconfig)
class VISIONOSRUNTIMESETTINGS_API UVisionOSRuntimeSettings : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	// Use VisionOS .ini files in the editor
	virtual const TCHAR* GetConfigOverridePlatform() const override 
	{
		return TEXT("VisionOS");
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostInitProperties() override;
#endif // WITH_EDITOR

    // When building for visionOS, default to Full Immersion mode.  If disabled, switches to Window mode. Note: when enabled,
	// the @main struct in UESwift.swift becomes the new main(). Plus, UIApplicationSceneManifest is auto-generation for Info.plist
    UPROPERTY(config, EditAnywhere, Category = Build, Meta = (DisplayName = "Enable visionOS's Immersion Mode (Experimental)", ConfigRestartRequired = true))
    bool bUseSwiftUIMain;

	// If checked, the Swift/ObjC bridging headers will be created.
	UPROPERTY(config, EditAnywhere, Category = Build, Meta = (DisplayName = "Create Swift/ObjC Bridging Headers", ConfigRestartRequired = true))
    bool bCreateBridgingHeader;
	
	// Specify the immersiveStyle we will launch into: 0 = .full, 1 = .mixed.
	UPROPERTY(config, EditAnywhere, Category = ImmersiveSpace, Meta = (DisplayName = "ImmersiveStyle"))
	int ImmersiveStyle = 0;
	
	// Specify the upperLimbVisibility we will start with: 0 = .hidden, 1 = .visible, 2 = .automatic.
	UPROPERTY(config, EditAnywhere, Category = ImmersiveSpace, Meta = (DisplayName = "UpperLimbVisibility"))
	int UpperLimbVisibility = 2;
};
