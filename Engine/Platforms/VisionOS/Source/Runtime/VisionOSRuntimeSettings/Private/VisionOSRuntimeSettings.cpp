// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisionOSRuntimeSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogVisionOSRuntimeSettings);

UVisionOSRuntimeSettings::UVisionOSRuntimeSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
void UVisionOSRuntimeSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
}

void UVisionOSRuntimeSettings::PostInitProperties()
{
	Super::PostInitProperties();
}
#endif // WITH_EDITOR

IMPLEMENT_MODULE(FDefaultModuleImpl, VisionOSRuntimeSettings);
